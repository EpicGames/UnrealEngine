{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "0.1",
  "FriendlyName": "Advanced Professional Video (APV) Decoder for Electra",
  "Description": "OpenAPV video decoder",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "Category": "Media Players",
  "EnabledByDefault": false,
  "CanContainContent": false,
  "IsBetaVersion": true,
  "IsExperimentalVersion": true,
  "Installed": false,
  "SupportedTargetPlatforms": [ "Win64" ],
  "Modules": [
    {
      "Name": "APVDecoderElectra",
      "Type": "Runtime",
      "LoadingPhase": "Default",
      "PlatformAllowList": [ "Win64" ],
      "PlatformArchitectureDenyList": [ "Win64:arm64" ],
      "TargetDenyList": [ "Server" ]
    }
  ],
  "Plugins": [
    {
      "Name": "ElectraCodecs",
      "Enabled": true
    }
  ]
}
