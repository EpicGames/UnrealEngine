// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class APVDecoderElectra : ModuleRules
	{
		public APVDecoderElectra(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
                    "Projects",
                    "ElectraBase",
                    "ElectraSamples",
					"ElectraCodecFactory",
					"ElectraDecoders",
                    "MP4Utilities"
                });

            if (Target.Platform == UnrealTargetPlatform.Win64)// || Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
            {
				PrivateDependencyModuleNames.Add("UEOpenAPV");
			}
		}
	}
}
