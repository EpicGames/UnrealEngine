// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaAPVDecoder.h"
#include "APVDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputVideo.h"
#include "ElectraDecodersUtils.h"
#include "Utils/OpenAPV/ElectraBitstreamProcessor_APV.h"
#include "Utils/OpenAPV/ElectraUtilsAPVVideo.h"
#include "MP4Utilities.h"

#include "ElectraDecodersPlatformResources.h"

PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START

#define OAPV_STATIC_DEFINE
#include "oapv/oapv.h"
#undef OAPV_STATIC_DEFINE

THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				2

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FVideoDecoderAPVElectra;


class FVideoDecoderOutputAPVElectra : public IElectraDecoderVideoOutput, public IElectraDecoderVideoOutputImageBuffers
{
public:
	virtual ~FVideoDecoderOutputAPVElectra()
	{
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	EOutputType GetOutputType() const
	{
		return EOutputType::Output;
	}

	int32 GetWidth() const override
	{
		return Width;
	}
	int32 GetHeight() const override
	{
		return Height;
	}
	int32 GetDecodedWidth() const override
	{
		return DecodedWidth;
	}
	int32 GetDecodedHeight() const override
	{
		return DecodedHeight;
	}
	FElectraVideoDecoderOutputCropValues GetCropValues() const override
	{
		return Crop;
	}
	int32 GetAspectRatioW() const override
	{
		return AspectW;
	}
	int32 GetAspectRatioH() const override
	{
		return AspectH;
	}
	int32 GetFrameRateNumerator() const override
	{
		return FrameRateN;
	}
	int32 GetFrameRateDenominator() const override
	{
		return FrameRateD;
	}
	int32 GetNumberOfBits() const override
	{
		return NumBits;
	}
	void GetExtraValues(TMap<FString, FVariant>& OutExtraValues) const override
	{
		OutExtraValues = ExtraValues;
	}
	void* GetPlatformOutputHandle(EElectraDecoderPlatformOutputHandleType InTypeOfHandle) const override
	{
		if (InTypeOfHandle == EElectraDecoderPlatformOutputHandleType::ImageBuffers)
		{
			return static_cast<IElectraDecoderVideoOutputImageBuffers*>(const_cast<FVideoDecoderOutputAPVElectra*>(this));
		}
		return nullptr;
	}
	IElectraDecoderVideoOutput::EImageCopyResult CopyPlatformImage(IElectraDecoderVideoOutputCopyResources* InCopyResources) const override
	{
		return IElectraDecoderVideoOutput::EImageCopyResult::NotSupported;
	}

	// Methods from IElectraDecoderVideoOutputImageBuffers
	uint32 GetCodec4CC() const override
	{
		return Codec4CC;
	}
	int32 GetNumberOfBuffers() const override
	{
		return NumBuffers;
	}
	TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> GetBufferDataByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].Buffer;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].Buffer;
		}
		return nullptr;
	}
	void* GetBufferTextureByIndex(int32 InBufferIndex) const override
	{
		return nullptr;
	}
	bool GetBufferTextureSyncByIndex(int32 InBufferIndex, FElectraDecoderOutputSync& SyncObject) const override
	{
		return false;
	}

	EPixelFormat GetBufferFormatByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].BufferFormat;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].BufferFormat;
		}
		return EPixelFormat::PF_Unknown;
	}
	EElectraTextureSamplePixelEncoding GetBufferEncodingByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].BufferEncoding;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].BufferEncoding;
		}
		return EElectraTextureSamplePixelEncoding::Native;
	}
	int32 GetBufferPitchByIndex(int32 InBufferIndex) const override
	{
		if (InBufferIndex == 0)
		{
			return Buffers[EBufferType::Buffer_Color].Pitch;
		}
		else if (InBufferIndex == 1)
		{
			return Buffers[EBufferType::Buffer_Alpha].Pitch;
		}
		return 0;
	}

public:
	FTimespan PTS;
	uint64 UserValue = 0;

	FElectraVideoDecoderOutputCropValues Crop;
	int32 DecodedWidth = 0;
	int32 DecodedHeight = 0;
	int32 Width = 0;
	int32 Height = 0;
	int32 NumBits = 0;
	int32 AspectW = 1;
	int32 AspectH = 1;
	int32 FrameRateN = 0;
	int32 FrameRateD = 0;
	TMap<FString, FVariant> ExtraValues;

	uint32 Codec4CC = 0;
	int32 NumBuffers = 0;

	enum EBufferType
	{
		Buffer_Color = 0,
		Buffer_Alpha = 1
	};

	struct FBufferInfo
	{
		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> Buffer;
		EPixelFormat BufferFormat;
		EElectraTextureSamplePixelEncoding BufferEncoding;
		int32 Pitch = 0;
	};
	FBufferInfo Buffers[2];
};


class FVideoDecoderAPVElectra : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
		OutOptions.Emplace(IElectraDecoderFeature::MinimumNumberOfOutputFrames, FVariant((int32)5));
		OutOptions.Emplace(IElectraDecoderFeature::IsAdaptive, FVariant(true));
	}

	FVideoDecoderAPVElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FVideoDecoderAPVElectra();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Video;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{
		TMap<FString, FVariant> DecoderFeatures;
		GetFeatures(DecoderFeatures);
		return FElectraDecoderBitstreamProcessorAPV::Create(DecoderFeatures, InitialCodecFormat, InitialCreationOptions);
	}

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	int32 CreateDecoder();
	void DestroyDecoder();

	Electra::FCodecTypeFormat InitialCodecFormat;
	TMap<FString, FVariant> InitialCreationOptions;
	int32 AspectW = 0;
	int32 AspectH = 0;

	IElectraDecoder::FError LastError;

	oapvd_t DecoderInstance = nullptr;
	oapvm_t DecoderMetaInstance = nullptr;

	TSharedPtr<FVideoDecoderOutputAPVElectra, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FAPVVideoDecoderElectraFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FAPVVideoDecoderElectraFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FAPVVideoDecoderElectraFactory()
	{ }

	FAPVVideoDecoderElectraFactory()
	{
		unsigned int Ver = 0;
		Version = ANSI_TO_TCHAR(oapv_version(&Ver));
		Name = TEXT("OpenAPV");
	}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FVideoDecoderAPVElectra::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Video || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
		{
			return 0;
		}
		switch(InCodecFormat.FourCC)
		{
			case ElectraDecodersUtil::Make4CC('a','p','v','1'):
			{
				if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
				{
					OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(FString(TEXT("Advanced Professional Video (APV)"))));
				}
				else
				{
					OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, InCodecFormat.HumanReadableFormatInfo);
				}
				OutFormatInfo.Emplace(IElectraDecoderFormatInfo::IsEveryFrameKeyframe, FVariant(true));
				return 1;
			}
			default:
			{
				break;
			}
		}
		return 0;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>() ? MakeShared<FVideoDecoderAPVElectra, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return Name; }
	FString GetVersion() const override
	{ return Version; }
	FString GetImplementation() const override
	{ return FString(); }
	FString GetVendor() const override
	{ return FString(); }

	FString Name;
	FString Version;

	static TSharedPtr<FAPVVideoDecoderElectraFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FAPVVideoDecoderElectraFactory, ESPMode::ThreadSafe> FAPVVideoDecoderElectraFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaAPVDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FAPVVideoDecoderElectraFactory::Self.IsValid());
	FAPVVideoDecoderElectraFactory::Self = MakeShared<FAPVVideoDecoderElectraFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FAPVVideoDecoderElectraFactory::Self.Get());
}

void FElectraMediaAPVDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FAPVVideoDecoderElectraFactory::Self.Get());
	FAPVVideoDecoderElectraFactory::Self.Reset();
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FVideoDecoderAPVElectra::FVideoDecoderAPVElectra(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
	: InitialCodecFormat(InCodecFormat)
	, InitialCreationOptions(InAdditionalOptions)
{
	if (InitialCodecFormat.Type == Electra::FCodecTypeFormat::EType::Video && InitialCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FVideo>())
	{
		const Electra::FCodecTypeFormat::FVideo& vid(InitialCodecFormat.Properties.Get<Electra::FCodecTypeFormat::FVideo>());
		AspectW = (int32) vid.AspectRatioW;
		AspectH = (int32) vid.AspectRatioH;
	}
}

FVideoDecoderAPVElectra::~FVideoDecoderAPVElectra()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

void FVideoDecoderAPVElectra::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FVideoDecoderAPVElectra::GetError() const
{
	return LastError;
}

bool FVideoDecoderAPVElectra::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FVideoDecoderAPVElectra::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FVideoDecoderAPVElectra::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	return IElectraDecoder::ECSDCompatibility::Compatible;
}

bool FVideoDecoderAPVElectra::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();
	DestroyDecoder();
	return true;
}


IElectraDecoder::EDecoderError FVideoDecoderAPVElectra::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	if (!DecoderInstance)
	{
		int32 Result = CreateDecoder();
		if (Result)
		{
			PostError(Result, TEXT("Failed to create APV decoder"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
	}

	// Decode data. This immediately produces a new output frame.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize > 4)
	{
		TArray<ElectraDecodersUtil::APVVideo::FFramePBUInfo> FramePBUs;
		ElectraDecodersUtil::APVVideo::ParseAUIntoFramePBUSubsamples(FramePBUs, MakeConstArrayView<uint8>(reinterpret_cast<const uint8*>(InInputAccessUnit.Data), InInputAccessUnit.DataSize));
		// We only decode the first frame PBU, whatever it is.
		if (FramePBUs.IsEmpty())
		{
			PostError(0, TEXT("Failed to parse APV access unit for frame PBUs"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		// Parse the frame information.
		ElectraDecodersUtil::APVVideo::FFrameInfo FrameInfo;
		if (!ElectraDecodersUtil::APVVideo::ParseFrameHeader(FrameInfo, MakeConstArrayView<uint8>(reinterpret_cast<const uint8*>(InInputAccessUnit.Data) + 4 + FramePBUs[0].PBUOffset, FramePBUs[0].PBUSize - 4)))
		{
			PostError(0, TEXT("Failed to parse first APV frame PBU in access unit"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		if (FrameInfo.BitDepthMinus8 != 2 || FrameInfo.ChromaFormatIDC != 2)
		{
			PostError(0, TEXT("Unsupported APV frame format (bitdepth or chroma format)"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		oapv_frms_t OutFrames;
		FMemory::Memzero(OutFrames);
		OutFrames.num_frms = 1;
		oapv_frm_t* of = &OutFrames.frm[0];

		// FIXME: We ignore the encoded color space and ask to get the output as 2 planes for now (see below)
		int ColorSpace = OAPV_CS_SET(OAPV_CF_PLANAR2, 10, 0);

		TSharedPtr<TArray64<uint8>, ESPMode::ThreadSafe> DecodedP010Buffer = MakeShared<TArray64<uint8>, ESPMode::ThreadSafe>();
		int32 OutputW = Align(FrameInfo.FrameWidth, OAPV_MB_W);
		int32 OutputH = Align(FrameInfo.FrameHeight, OAPV_MB_H);
		DecodedP010Buffer->AddUninitialized(OutputW * OutputH * sizeof(uint16) * 2);

		oapv_imgb_t ib;
		FMemory::Memzero(ib);
		ib.w[0] = FrameInfo.FrameWidth;
		ib.h[0] = FrameInfo.FrameHeight;
		int bd = OAPV_CS_GET_BYTE_DEPTH(ColorSpace);
		// Setup for 2 planes
		ib.w[1] = ib.w[0];
		ib.h[1] = ib.h[0];
		ib.np = 2;
		uint8* P010Destination = DecodedP010Buffer->GetData();
		for(int32 i=0; i<ib.np; ++i)
		{
			ib.aw[i] = Align(ib.w[i], OAPV_MB_W);
			ib.ah[i] = Align(ib.h[i], OAPV_MB_H);
			ib.s[i] = ib.aw[i] * bd;
			ib.e[i] = ib.ah[i];
			ib.bsize[i] = ib.s[i] * ib.e[i];
			ib.a[i] = ib.baddr[i] = P010Destination;
			P010Destination += ib.bsize[i];
		}
		ib.cs = ColorSpace;
		of->imgb = &ib;
		if (!of->imgb)
		{
			PostError(0, TEXT("APV decoding failed, out of memory"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		oapv_bitb_t InputBS;
		FMemory::Memzero(InputBS);
		// The decoder checks for the `aPv1` magic which may not be present if we decode one sub frame from
		// the access unit. In this case we have to create a temporary buffer to prepend the magic.
		uint8* TempAU = nullptr;
		bool bHaveMagic = FramePBUs[0].PBUOffset >= 4 && *reinterpret_cast<const uint32*>(reinterpret_cast<const uint8*>(InInputAccessUnit.Data) + FramePBUs[0].PBUOffset - 4) == MP4Utilities::EndianSwap((uint32)0x61507631U);
		if (!bHaveMagic)
		{
			TempAU = reinterpret_cast<uint8*>(FMemory::Malloc(FramePBUs[0].PBUSize + 4));
			if (!TempAU)
			{
				PostError(0, TEXT("APV decoding failed, out of memory"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
			}
			FMemory::Memcpy(TempAU + 4, reinterpret_cast<const uint8*>(InInputAccessUnit.Data) + FramePBUs[0].PBUOffset, FramePBUs[0].PBUSize);
			*reinterpret_cast<uint32*>(TempAU) = MP4Utilities::EndianSwap((uint32)0x61507631U);
			InputBS.addr = TempAU;
			InputBS.ssize = FramePBUs[0].PBUSize + 4;
		}
		else if (FramePBUs[0].PBUOffset >= 4)
		{
			InputBS.addr = (void*)(reinterpret_cast<const uint8*>(InInputAccessUnit.Data) + FramePBUs[0].PBUOffset - 4);
			InputBS.ssize = FramePBUs[0].PBUSize + 4;
		}
		else
		{
			PostError(0, TEXT("Failed to decode APV. Malformed PBU?"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		oapvd_stat_t Stats;
		FMemory::Memzero(Stats);
        int Result = oapvd_decode(DecoderInstance, &InputBS, &OutFrames, DecoderMetaInstance, &Stats);
		if (TempAU)
		{
			FMemory::Free(TempAU);
			TempAU = nullptr;
		}

		if (OAPV_FAILED(Result))
		{
			PostError(Result, TEXT("APV decoding failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}

		// Handle metadata
		int num_plds = 0; // number of metadata payload
		Result = oapvm_get_all(DecoderMetaInstance, nullptr, &num_plds);
		if (OAPV_FAILED(Result))
		{
			PostError(Result, TEXT("APV decoding failed to get metadata"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return IElectraDecoder::EDecoderError::Error;
		}
		else if (num_plds > 0)
		{
			oapvm_payload_t *pld = new oapvm_payload_t[num_plds];
			Result = oapvm_get_all(DecoderMetaInstance, pld, &num_plds);
			if (OAPV_FAILED(Result))
			{
				delete [] pld;
				pld = nullptr;
				PostError(Result, TEXT("APV decoding failed to get metadata"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
				return IElectraDecoder::EDecoderError::Error;
            }
			else
			{
			}
			if (pld)
			{
				delete [] pld;
			}
		}

		int32 DisplayWidth = (int32)FrameInfo.FrameWidth;
		int32 DisplayHeight = (int32)FrameInfo.FrameHeight;

		TSharedPtr<FVideoDecoderOutputAPVElectra, ESPMode::ThreadSafe> NewOutput = MakeShared<FVideoDecoderOutputAPVElectra>();
		NewOutput->PTS = InInputAccessUnit.PTS;
		NewOutput->UserValue = InInputAccessUnit.UserValue;

		NewOutput->DecodedWidth = OutputW;
		NewOutput->DecodedHeight = OutputH * 3/2;
		NewOutput->Width = DisplayWidth;
		NewOutput->Height = DisplayHeight;

		NewOutput->Crop.Right = OutputW - DisplayWidth;
		NewOutput->Crop.Bottom = OutputH - DisplayHeight;

		if (AspectW && AspectH)
		{
			NewOutput->AspectW = AspectW;
			NewOutput->AspectH = AspectH;
		}

		NewOutput->NumBits = 10;

		// FIXME: For now we remove every other line in the UV plane to convert from 4:2:2 (P210) to 4:2:0 (P010)
		uint8* const UVPlane = DecodedP010Buffer->GetData() + OutputW * OutputH * 2;
		for(int32 v=1; v<OutputH / 2; ++v)
		{
			const uint8* SrcUV = UVPlane + v * 2 * OutputW * sizeof(uint16);
			uint8* DstUV = UVPlane + v * OutputW * sizeof(uint16);
			FMemory::Memcpy(DstUV, SrcUV, OutputW * sizeof(uint16));
		}

		NewOutput->NumBuffers = 1;
		NewOutput->Buffers[0].Buffer = MoveTemp(DecodedP010Buffer);
		NewOutput->Buffers[0].BufferFormat = EPixelFormat::PF_P010;
		NewOutput->Buffers[0].BufferEncoding = EElectraTextureSamplePixelEncoding::Native;
		NewOutput->Buffers[0].Pitch = OutputW * sizeof(uint16);

		NewOutput->Codec4CC = InitialCodecFormat.FourCC;
		NewOutput->ExtraValues.Emplace(TEXT("codec"), FVariant(TEXT("apv")));
		NewOutput->ExtraValues.Emplace(TEXT("codec_4cc"), FVariant(InitialCodecFormat.FourCC));

		CurrentOutput = MoveTemp(NewOutput);
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderAPVElectra::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FVideoDecoderAPVElectra::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FVideoDecoderAPVElectra::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FVideoDecoderAPVElectra::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}

int32 FVideoDecoderAPVElectra::CreateDecoder()
{
	oapvd_cdesc_t DecoderDescriptor;
	DecoderDescriptor.threads = 16;
	int Result = 0;
	DecoderInstance = oapvd_create(&DecoderDescriptor, &Result);
	if (!DecoderInstance)
	{
		return Result;
	}
	Result = 0;
	DecoderMetaInstance = oapvm_create(&Result);
	return Result;
}

void FVideoDecoderAPVElectra::DestroyDecoder()
{
	if (DecoderMetaInstance)
	{
		oapvm_delete(DecoderMetaInstance);
		DecoderMetaInstance = nullptr;
	}
	if (DecoderInstance)
	{
		oapvd_delete(DecoderInstance);
		DecoderInstance = nullptr;
	}
}
