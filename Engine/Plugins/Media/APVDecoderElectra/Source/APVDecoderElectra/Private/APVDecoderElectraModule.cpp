// Copyright Epic Games, Inc. All Rights Reserved.

#include "APVDecoderElectraModule.h"
#include "APVDecoder/ElectraMediaAPVDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "APVDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogAPVElectraDecoder);

class FAPVElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaAPVDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaAPVDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FAPVElectraDecoderModule, APVDecoderElectra);

#undef LOCTEXT_NAMESPACE
